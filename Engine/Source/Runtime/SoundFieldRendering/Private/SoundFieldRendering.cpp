// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundFieldRendering.h"
#include "Engine/Engine.h"
#include "HAL/IConsoleManager.h"
#include "DSP/Dsp.h"
#include "AudioMixer.h"
#include "AudioMixerDevice.h"
#include "DSP/BufferVectorOperations.h"


FSoundFieldDecoder::FSoundFieldDecoder()
{
	// initialize virtual speaker world locked gain map (for first order)
	constexpr const float DEG_2_RAD = PI / 180.0f;
	const int32 NumVirtualChannels = 9;
	const int32 NumAmbiChannels = 4;
	
	// gain normalization factor
	const float G = FMath::Sqrt(1.0f / 7.0f); // 7 final virtual horizontal speakers (excludes LFE)

	// initialize gains buffer
	FoaVirtualSpeakerWordLockedGains.Reset();
	FoaVirtualSpeakerWordLockedGains.AddZeroed(NumVirtualChannels * NumAmbiChannels);
	float* RESTRICT SpeakerGainsPtr = FoaVirtualSpeakerWordLockedGains.GetData();
	
	for (int32 OutChan = 0; OutChan < NumVirtualChannels; ++OutChan)
	{
		const Audio::FChannelPositionInfo& CurrSpeakerPos = VirtualSpeakerLocationsHorzOnly[OutChan];

		// (skip LFE)
		if (CurrSpeakerPos.Channel == EAudioMixerChannel::LowFrequency)
		{
			SpeakerGainsPtr += NumAmbiChannels;
			continue;
		}

		const float Azimuth = CurrSpeakerPos.Azimuth * DEG_2_RAD;
		const float Elevation = CurrSpeakerPos.Elevation * DEG_2_RAD;

		// SpeakerGainsPtr is assumed to be a pointer to a 4-element array
		FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(1/*first order*/, Azimuth, Elevation, SpeakerGainsPtr);

		// apply gain normalization factor directly to speaker gains
		for (int i = 0; i < NumAmbiChannels; ++i)
		{
			SpeakerGainsPtr[i] *= G;
		}
		SpeakerGainsPtr += NumAmbiChannels;
	}

	// reset speaker gains pointer
	SpeakerGainsPtr = FoaVirtualSpeakerWordLockedGains.GetData();

	// adjust gains to sum front center->front L/R and rear center->back L/R
	constexpr const int32 FrontRightOffset  = NumAmbiChannels * EAudioMixerChannel::FrontRight;
	constexpr const int32 FrontCenterOffset = NumAmbiChannels * EAudioMixerChannel::FrontCenter;
	constexpr const int32 BackLeftOffest    = NumAmbiChannels * EAudioMixerChannel::BackLeft;
	constexpr const int32 BackRightOffset   = NumAmbiChannels * EAudioMixerChannel::BackRight;
	constexpr const int32 BackCenterOffset  = NumAmbiChannels * EAudioMixerChannel::BackCenter;

	VectorRegister FrontLeftGains   = VectorLoadAligned(SpeakerGainsPtr);
	VectorRegister FrontRightGains  = VectorLoadAligned(SpeakerGainsPtr + FrontRightOffset);
	VectorRegister FrontCenterGains = VectorLoadAligned(SpeakerGainsPtr + FrontCenterOffset);
	VectorRegister BackLeftGains    = VectorLoadAligned(SpeakerGainsPtr + BackLeftOffest);
	VectorRegister BackRightGains   = VectorLoadAligned(SpeakerGainsPtr + BackRightOffset);
	VectorRegister BackCenterGains  = VectorLoadAligned(SpeakerGainsPtr + BackCenterOffset);

	FrontLeftGains  = VectorMultiplyAdd(Sqrt2Over2Vec, FrontCenterGains, FrontLeftGains);
	FrontRightGains = VectorMultiplyAdd(Sqrt2Over2Vec, FrontCenterGains, FrontRightGains);
	VectorStoreAligned(FrontLeftGains, SpeakerGainsPtr);
	VectorStoreAligned(FrontRightGains, SpeakerGainsPtr + FrontRightOffset);

	BackLeftGains  = VectorMultiplyAdd(Sqrt2Over2Vec, BackCenterGains, BackLeftGains);
	BackRightGains = VectorMultiplyAdd(Sqrt2Over2Vec, BackCenterGains, BackRightGains);
	VectorStoreAligned(BackLeftGains, SpeakerGainsPtr + BackLeftOffest);
	VectorStoreAligned(BackRightGains, SpeakerGainsPtr + BackRightOffset);

	VectorStoreAligned(VectorZero(), SpeakerGainsPtr + FrontCenterOffset);
	VectorStoreAligned(VectorZero(), SpeakerGainsPtr + BackCenterOffset);

	TargetSpeakerGains = FoaVirtualSpeakerWordLockedGains;
}

void FSoundFieldDecoder::DecodeAudioDirectlyToDeviceOutputPositions(const FAmbisonicsSoundfieldBuffer& InputData, const FSoundfieldSpeakerPositionalData& OutputPositions, Audio::FAlignedFloatBuffer& OutputData)
{
	if (InputData.NumChannels == 0 || InputData.AudioBuffer.Num() == 0)
	{
		FMemory::Memzero(OutputData.GetData(), OutputData.Num() * sizeof(float));
		return;
	}

	constexpr const float DEG_2_RAD = PI / 180.0f;

	const int32 NumFrames = InputData.AudioBuffer.Num() / InputData.NumChannels;
	const int32 NumOutputChannels = OutputPositions.NumChannels;
	const int32 NumAmbiChannels = InputData.NumChannels;

	int32 AmbiOrder = 0;
	if (NumAmbiChannels == 4)
	{
		AmbiOrder = 1;
	}
	else if (NumAmbiChannels == 9)
	{
		AmbiOrder = 2;
	}
	else if (NumAmbiChannels == 16)
	{
		AmbiOrder = 3;
	}
	else if (NumAmbiChannels == 25)
	{
		AmbiOrder = 4;
	}
	else if (NumAmbiChannels == 36)
	{
		AmbiOrder = 5;
	}
	else
	{
		// Unsupported ambisonics order.
		checkNoEntry();
	}

	check(NumOutputChannels > 0);
	check(OutputData.Num() == NumFrames * NumOutputChannels);

	CurrentSpeakerGains = TargetSpeakerGains;
	TargetSpeakerGains.Reset();
	TargetSpeakerGains.AddZeroed(NumOutputChannels * NumAmbiChannels);

	// Prepare vector registers and pointers
	
	float* RESTRICT SpeakerGainsPtr = TargetSpeakerGains.GetData();
	float* RESTRICT OutputBufferPtrBuffer = OutputData.GetData();
	const float* RESTRICT pAmbiFrame = InputData.AudioBuffer.GetData();

	// Zero output audio before summing into it
	FMemory::Memzero(OutputData.GetData(), sizeof(float) * OutputData.Num());

	// gain normalization factor
	const float G = FMath::Sqrt(1.0f / static_cast<float>(NumOutputChannels));


	// obtain listener orientation in spherical coordinates
	FVector2D ListenerRotationSphericalCoord = OutputPositions.Rotation.Vector().UnitCartesianToSpherical();
	FSphericalHarmonicCalculator::AdjustUESphericalCoordinatesForAmbisonics(ListenerRotationSphericalCoord);
	check(OutputPositions.ChannelPositions);

	// Don't interpolate if the number of speakers has changed, or if listener rotation has not changed
	bool bShouldInterpolate = TargetSpeakerGains.Num() == CurrentSpeakerGains.Num();
	bShouldInterpolate |= !FMath::IsNearlyEqual(ListenerRotationSphericalCoord.X, LastListenerRotationSphericalCoord.X);
	bShouldInterpolate |= !FMath::IsNearlyEqual(ListenerRotationSphericalCoord.Y, LastListenerRotationSphericalCoord.Y);

	// fill out the ambisonic speaker gain maps
	for (int32 OutChan = 0; OutChan < NumOutputChannels; ++OutChan)
	{

		const Audio::FChannelPositionInfo& CurrSpeakerPos = (*OutputPositions.ChannelPositions)[OutChan];

		// skip LFE and Center channel (leave gains at zero)
		if (CurrSpeakerPos.Channel == EAudioMixerChannel::LowFrequency || CurrSpeakerPos.Channel == EAudioMixerChannel::FrontCenter)
		{
			SpeakerGainsPtr += NumAmbiChannels;
			continue;
		}

		const float Theta = ListenerRotationSphericalCoord.X - CurrSpeakerPos.Azimuth * DEG_2_RAD; // azimuth
		const float Phi = ListenerRotationSphericalCoord.Y - CurrSpeakerPos.Elevation * DEG_2_RAD; // elevation

		// SpeakerGainsPtr is assumed to be a pointer to a NumAmbiChannels-element array
		FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(AmbiOrder, Theta, Phi, SpeakerGainsPtr);

		// apply gain normalization factor directly to speaker gains
		for (int i = 0; i < NumAmbiChannels; ++i)
		{
			SpeakerGainsPtr[i] *= G;
		}

		SpeakerGainsPtr += NumAmbiChannels;
	}

	// reset speaker gains pointer
	SpeakerGainsPtr = TargetSpeakerGains.GetData();


	if (NumAmbiChannels == 4)
	{
		FirstOrderDecodeLoop(NumFrames, NumOutputChannels, pAmbiFrame, NumAmbiChannels, SpeakerGainsPtr, OutputBufferPtrBuffer);
	}
	else if (NumAmbiChannels % 4 == 0)
	{
		// All odd-ordered ambisonics has a channel count divisible by 4, and thus can be fully vectorized.
		OddOrderDecodeLoop(NumFrames, NumOutputChannels, pAmbiFrame, NumAmbiChannels, SpeakerGainsPtr, OutputBufferPtrBuffer);
	}
	else
	{
		EvenOrderDecodeLoop(NumFrames, NumOutputChannels, pAmbiFrame, NumAmbiChannels, SpeakerGainsPtr, OutputBufferPtrBuffer);
	}
	
}

void FSoundFieldDecoder::FirstOrderDecodeLoop(const int32 NumFrames, const int32 NumOutputChannels, const float* pAmbiFrame, const int32 NumAmbiChannels, float* SpeakerGainsPtr, float* OutputBufferPtrBuffer)
{
	VectorRegister CurrSpeakerGain = VectorRegister();
	VectorRegister CurrAmbiFrame = VectorRegister();

	for (int32 OutFrame = 0; OutFrame < NumFrames; ++OutFrame)
	{
		CurrAmbiFrame = VectorLoadAligned(pAmbiFrame + OutFrame * NumAmbiChannels);

		// for each output frame...
		for (int32 OutChannel = 0; OutChannel < NumOutputChannels; ++OutChannel)
		{
			const int32 FrameOffset = OutFrame * NumOutputChannels;
			CurrSpeakerGain = VectorLoadAligned(SpeakerGainsPtr + (OutChannel * NumAmbiChannels));
			OutputBufferPtrBuffer[FrameOffset + OutChannel] += VectorGetComponent(VectorDot4(CurrAmbiFrame, CurrSpeakerGain), 0);
		}
	}
}

void FSoundFieldDecoder::OddOrderDecodeLoop(const int32 NumFrames, const int32 NumOutputChannels, const float* pAmbiFrame, const int32 NumAmbiChannels, float* SpeakerGainsPtr, float* OutputBufferPtrBuffer)
{
	VectorRegister CurrSpeakerGain = VectorRegister();
	VectorRegister CurrAmbiVector = VectorRegister();

	const int32 NumAmbiVectors = NumAmbiChannels / 4;

	for (int32 OutFrame = 0; OutFrame < NumFrames; ++OutFrame)
	{
		const int32 FrameOffset = OutFrame * NumOutputChannels;
		const int32 AmbiFrameOffset = (OutFrame * NumAmbiChannels);

		for (int32 AmbiVectorOffset = 0; AmbiVectorOffset < NumAmbiChannels; AmbiVectorOffset+= 4)
		{
			CurrAmbiVector = VectorLoadAligned(pAmbiFrame + AmbiFrameOffset + AmbiVectorOffset);

			// for each output channel in this frame...
			for (int32 Channel = 0; Channel < NumOutputChannels; ++Channel)
			{
				CurrSpeakerGain = VectorLoadAligned(SpeakerGainsPtr + (Channel * NumAmbiChannels) + AmbiVectorOffset);
				OutputBufferPtrBuffer[FrameOffset + Channel] += VectorGetComponent(VectorDot4(CurrAmbiVector, CurrSpeakerGain), 0);
			}
		}
	}
}

void FSoundFieldDecoder::EvenOrderDecodeLoop(const int32 NumFrames, const int32 NumOutputChannels, const float* pAmbiFrame, const int32 NumAmbiChannels, float* SpeakerGainsPtr, float* OutputBufferPtrBuffer)
{
	VectorRegister CurrSpeakerGain = VectorRegister();
	VectorRegister CurrAmbiVector = VectorRegister();

	const int32 NumAmbiVectors = NumAmbiChannels / 4;
	const int32 VectorizedAmbiChannels = NumAmbiVectors * 4;

	for (int32 OutFrame = 0; OutFrame < NumFrames; ++OutFrame)
	{
		const int32 FrameOffset = OutFrame * NumOutputChannels;
		const int32 AmbiFrameOffset = (OutFrame * NumAmbiChannels);

		for (int32 AmbiVectorOffset = 0; AmbiVectorOffset < VectorizedAmbiChannels; AmbiVectorOffset += 4)
		{
			CurrAmbiVector = VectorLoad(pAmbiFrame + AmbiFrameOffset + AmbiVectorOffset);

			// for each output channel in this frame...
			for (int32 Channel = 0; Channel < NumOutputChannels; ++Channel)
			{
				CurrSpeakerGain = VectorLoad(SpeakerGainsPtr + (Channel * NumAmbiChannels) + AmbiVectorOffset);
				OutputBufferPtrBuffer[FrameOffset + Channel] += VectorGetComponent(VectorDot4(CurrAmbiVector, CurrSpeakerGain), 0);
			}
		}

		// Handle the remaining ambi channel(s) that don't fill a full vector.
		for (int32 AmbiCh = VectorizedAmbiChannels; AmbiCh < NumAmbiChannels; ++AmbiCh)
		{
			const float AmbiSample = pAmbiFrame[AmbiFrameOffset + AmbiCh];

			for (int32 Channel = 0; Channel < NumOutputChannels; ++Channel)
			{
				const float SpeakerGain = SpeakerGainsPtr[(Channel * NumAmbiChannels) + AmbiCh];
				OutputBufferPtrBuffer[FrameOffset + Channel] += SpeakerGain * AmbiSample;
			}
		}
	}
}


// this will probably be the use case for 5.1, prioritize optimizations here
void FSoundFieldDecoder::DecodeAudioToSevenOneAndDownmixToDevice(const FAmbisonicsSoundfieldBuffer& InputData, const FSoundfieldSpeakerPositionalData& OutputPositions, Audio::FAlignedFloatBuffer& OutputData)
{
	if (InputData.NumChannels == 0 || InputData.AudioBuffer.Num() == 0)
	{
		FMemory::Memzero(OutputData.GetData(), OutputData.Num() * sizeof(float));
		return;
	}

	constexpr const float DEG_2_RAD = PI / 180.0f;

	const int32 NumFrames = InputData.AudioBuffer.Num() / InputData.NumChannels;
	const int32 NumOutputSamples = OutputData.Num();
	const int32 NumAmbiChannels = InputData.NumChannels;
	const int32 NumVirtualOutputSamples = NumFrames * 8;

	const int32 NumOutputChannels = OutputPositions.NumChannels;
	const int32 NumVirtualChannels = VirtualSpeakerLocationsHorzOnly.Num();

	int32 AmbiOrder = 0;
	if (NumAmbiChannels == 4)
	{
		AmbiOrder = 1;
	}
	else if (NumAmbiChannels == 9)
	{
		AmbiOrder = 2;
	}
	else if (NumAmbiChannels == 16)
	{
		AmbiOrder = 3;
	}
	else if (NumAmbiChannels == 25)
	{
		AmbiOrder = 4;
	}
	else if (NumAmbiChannels == 36)
	{
		AmbiOrder = 5;
	}
	else
	{
		// Unsupported ambisonics order.
		checkNoEntry();
	}


	const bool bDecodeDirectToOutBuffer = (NumOutputChannels == (NumVirtualChannels - 1));
	check(NumVirtualChannels == 9); // if this has changed, it breaks our assumptions for fast gain indexing

	// get listener orientation in spherical coordinates
	FQuat RelativeRotation = OutputPositions.Rotation * InputData.Rotation;
	FVector2D ListenerRotationSphericalCoord = RelativeRotation.Vector().UnitCartesianToSpherical();
	FSphericalHarmonicCalculator::AdjustUESphericalCoordinatesForAmbisonics(ListenerRotationSphericalCoord);

	// Set up output buffer for virtual speaker positions
	VirtualSpeakerScratchBuffers.Reset();
	VirtualSpeakerScratchBuffers.AddZeroed(NumFrames * 8);

	// Don't interpolate if the listener rotation has not changed
	bool bShouldInterpolate = !FMath::IsNearlyEqual(ListenerRotationSphericalCoord.X, LastListenerRotationSphericalCoord.X);
	bShouldInterpolate |= !FMath::IsNearlyEqual(ListenerRotationSphericalCoord.Y, LastListenerRotationSphericalCoord.Y);

	if (bShouldInterpolate)
	{
		CurrentSpeakerGains = TargetSpeakerGains;
		TargetSpeakerGains = FoaVirtualSpeakerWordLockedGains;
		FVector ListenerRot = RelativeRotation.Euler();
		FoaRotationInPlace(TargetSpeakerGains, ListenerRot.X, ListenerRot.Y, ListenerRot.Z);

		// If we just switched to this decode method (from direct to device), we need to initialize CurrentSpeakerGains size
		if (CurrentSpeakerGains.Num() != TargetSpeakerGains.Num())
		{
			CurrentSpeakerGains = TargetSpeakerGains;
			bShouldInterpolate = false;
		}
	}

	// Prepare vector registers, pointers, and index masks
	float* RESTRICT SpeakerGainsPtr = bShouldInterpolate? CurrentSpeakerGains.GetData() : TargetSpeakerGains.GetData();
	const float* RESTRICT AmbiInputBufferPtr = InputData.AudioBuffer.GetData();
	float* RESTRICT OutputBufferPtr = bDecodeDirectToOutBuffer ? OutputData.GetData() : VirtualSpeakerScratchBuffers.GetData();
	VectorRegister CurrSpeakerGain = VectorRegister();
	VectorRegister CurrAmbiFrame = VectorRegister();
	constexpr const int32 OutputChannelIndexWrapMask = 0x00000007;
	constexpr const int32 AmbiChannelIndexWrapMask = ~0x3; // 8 virtual output channels, so to mod we clear out all but the 2 LS bits

	if (!bShouldInterpolate) // Non-Interpolated decode
	{
		// for each output frame...
		for (uint64 OutSample = 0; OutSample < NumVirtualOutputSamples; ++OutSample)
		{
			const uint64 ChannelOffset = OutSample & OutputChannelIndexWrapMask;

			// (non bit-shift versions:)
			// CurrAmbiFrame = VectorLoadAligned(AmbiInputBufferPtr + OutSample / 8 * NumAmbiChannels);
			// CurrSpeakerGain = VectorLoadAligned(SpeakerGainsPtr + ChannelOffset * NumAmbiChannels);
			CurrAmbiFrame = VectorLoadAligned(AmbiInputBufferPtr + ((OutSample >> 1) & AmbiChannelIndexWrapMask));
			CurrSpeakerGain = VectorLoadAligned(SpeakerGainsPtr + (ChannelOffset << 2));

			OutputBufferPtr[OutSample] = VectorGetComponent(VectorDot4(CurrAmbiFrame, CurrSpeakerGain), 0);
		}
	}
	else // Interpolated decode
	{
		SpeakerGainLerper.Init(CurrentSpeakerGains, TargetSpeakerGains, NumFrames);
		const Audio::FAlignedFloatBuffer& SpeakerGainLerpDeltas = SpeakerGainLerper.GetDeltaBuffer();
		const float* RESTRICT SpeakerGainLerpDeltasPtr = SpeakerGainLerpDeltas.GetData();
		VectorRegister CurrSpeakerGainDelta = VectorRegister();

		// for each output frame...
		for (uint64 OutSample = 0; OutSample < NumVirtualOutputSamples; ++OutSample)
		{
			const uint64 ChannelOffset = OutSample & OutputChannelIndexWrapMask;
			const uint64 SpeakerGainOffset = ChannelOffset << 2;

			// (non bit-shift versions:)
			// CurrAmbiFrame = VectorLoadAligned(AmbiInputBufferPtr + OutSample / 8 * NumAmbiChannels);
			// CurrSpeakerGain = VectorLoadAligned(SpeakerGainsPtr + ChannelOffset * NumAmbiChannels);
			CurrAmbiFrame = VectorLoadAligned(AmbiInputBufferPtr + ((OutSample >> 1) & AmbiChannelIndexWrapMask));
			CurrSpeakerGain = VectorLoadAligned(SpeakerGainsPtr + SpeakerGainOffset);
			CurrSpeakerGainDelta = VectorLoadAligned(SpeakerGainLerpDeltasPtr + SpeakerGainOffset);

			OutputBufferPtr[OutSample] = VectorGetComponent(VectorDot4(CurrAmbiFrame, CurrSpeakerGain), 0);

			// step forward gain interpolation
			VectorRegister UpdatedSpeakerGain = VectorAdd(CurrSpeakerGain, CurrSpeakerGainDelta);
			VectorStoreAligned(UpdatedSpeakerGain, SpeakerGainsPtr + SpeakerGainOffset);
		}
	}

	// downmix to output format if it is not 7.1
	if (!bDecodeDirectToOutBuffer)
	{
		FMemory::Memzero(OutputData.GetData(), OutputData.Num() * sizeof(float));
		Audio::FMixerDevice::Get2DChannelMap(false, 8, NumOutputChannels, false, MixdownGainsMap);
		Audio::DownmixAndSumIntoBuffer(8, NumOutputChannels, VirtualSpeakerScratchBuffers, OutputData, MixdownGainsMap.GetData());
	}

	LastListenerRotationSphericalCoord = ListenerRotationSphericalCoord;
}

void FSoundFieldDecoder::RotateFirstOrderAmbisonicsBed(const FAmbisonicsSoundfieldBuffer& InputData, FAmbisonicsSoundfieldBuffer& OutputData, const FQuat& DestinationRotation, const FQuat& PreviousRotation)
{
	check(InputData.NumChannels == 4); // this function assumes FOA
	const int32 NumFrames = InputData.AudioBuffer.Num() / InputData.NumChannels;

	VectorRegister InputAmbiFrame = VectorRegister();
	const float* RESTRICT InputBufferPtr = InputData.AudioBuffer.GetData();
	float* RESTRICT OutputBufferPtr = OutputData.AudioBuffer.GetData();

	FVector CurrentEulerRotationVector = PreviousRotation.Euler();
	FVector TargetEulerRotationVector = DestinationRotation.Euler();

	// Check to see if rotation has changed (to avoid interpolation)
	bool bHasRotated = !FMath::IsNearlyEqual(CurrentEulerRotationVector.X, TargetEulerRotationVector.X);
	bHasRotated |= !FMath::IsNearlyEqual(CurrentEulerRotationVector.Y, TargetEulerRotationVector.Y);
	bHasRotated |= !FMath::IsNearlyEqual(CurrentEulerRotationVector.Z, TargetEulerRotationVector.Z);

	// Obtain our rotation matrix
	FMatrix TargetRotationMatrix;
	SphereHarmCalc.GenerateFirstOrderRotationMatrixGivenDegrees(TargetEulerRotationVector.X, TargetEulerRotationVector.Y, TargetEulerRotationVector.Z, TargetRotationMatrix);

	VectorRegister RotMatrixRow0 = VectorLoadAligned(TargetRotationMatrix.M[0]);
	VectorRegister RotMatrixRow1 = VectorLoadAligned(TargetRotationMatrix.M[1]);
	VectorRegister RotMatrixRow2 = VectorLoadAligned(TargetRotationMatrix.M[2]);
	VectorRegister RotMatrixRow3 = VectorLoadAligned(TargetRotationMatrix.M[3]);

	// treat the input and output ambi frames as column vectors and multiply:
	// (i.e. OutputFrame[i] = TargetRotationMatrix * InputFrame[i])
	if (!bHasRotated) // no interpolation needed
	{
		for (int32 i = 0; i < NumFrames; ++i)
		{
			InputAmbiFrame = VectorLoadAligned(InputBufferPtr);

			OutputBufferPtr[0] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow0), 0);
			OutputBufferPtr[1] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow1), 0);
			OutputBufferPtr[2] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow2), 0);
			OutputBufferPtr[3] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow3), 0);

			OutputBufferPtr += 4;
			InputBufferPtr += 4;
		}
	}
	else
	{
		FMatrix CurrentRotationMatrix;

		// we already have our target matrix above.
		// now we calculate our "current" matrix (where we left off)
		SphereHarmCalc.GenerateFirstOrderRotationMatrixGivenDegrees(CurrentEulerRotationVector.X, CurrentEulerRotationVector.Y, CurrentEulerRotationVector.Z, CurrentRotationMatrix);

		const float NumFramesFloat = static_cast<float>(NumFrames);
		VectorRegister NumFramesVec = MakeVectorRegister(NumFramesFloat, NumFramesFloat, NumFramesFloat, NumFramesFloat);

		// Find our delta...
		VectorRegister RotMatrixDeltaRow0 = VectorLoadAligned(CurrentRotationMatrix.M[0]);
		VectorRegister RotMatrixDeltaRow1 = VectorLoadAligned(CurrentRotationMatrix.M[1]);
		VectorRegister RotMatrixDeltaRow2 = VectorLoadAligned(CurrentRotationMatrix.M[2]);
		VectorRegister RotMatrixDeltaRow3 = VectorLoadAligned(CurrentRotationMatrix.M[3]);

		RotMatrixDeltaRow0 = VectorSubtract(RotMatrixRow0, RotMatrixDeltaRow0);
		RotMatrixDeltaRow1 = VectorSubtract(RotMatrixRow1, RotMatrixDeltaRow1);
		RotMatrixDeltaRow2 = VectorSubtract(RotMatrixRow2, RotMatrixDeltaRow2);
		RotMatrixDeltaRow3 = VectorSubtract(RotMatrixRow3, RotMatrixDeltaRow3);

		RotMatrixDeltaRow0 = VectorDivide(RotMatrixDeltaRow0, NumFramesVec);
		RotMatrixDeltaRow1 = VectorDivide(RotMatrixDeltaRow1, NumFramesVec);
		RotMatrixDeltaRow2 = VectorDivide(RotMatrixDeltaRow2, NumFramesVec);
		RotMatrixDeltaRow3 = VectorDivide(RotMatrixDeltaRow3, NumFramesVec);

		// no longer need the target matrix, so load the current matrix into registers
		RotMatrixRow0 = VectorLoadAligned(CurrentRotationMatrix.M[0]);
		RotMatrixRow1 = VectorLoadAligned(CurrentRotationMatrix.M[1]);
		RotMatrixRow2 = VectorLoadAligned(CurrentRotationMatrix.M[2]);
		RotMatrixRow3 = VectorLoadAligned(CurrentRotationMatrix.M[3]);

		for (int32 i = 0; i < NumFrames; ++i)
		{
			// load input frame
			InputAmbiFrame = VectorLoadAligned(InputBufferPtr);

			// perform Matrix*Vector
			OutputBufferPtr[0] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow0), 0);
			OutputBufferPtr[1] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow1), 0);
			OutputBufferPtr[2] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow2), 0);
			OutputBufferPtr[3] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow3), 0);

			// linear interpolation step
			RotMatrixRow0 = VectorAdd(RotMatrixRow0, RotMatrixDeltaRow0);
			RotMatrixRow1 = VectorAdd(RotMatrixRow1, RotMatrixDeltaRow1);
			RotMatrixRow2 = VectorAdd(RotMatrixRow2, RotMatrixDeltaRow2);
			RotMatrixRow3 = VectorAdd(RotMatrixRow3, RotMatrixDeltaRow3);

			OutputBufferPtr += 4;
			InputBufferPtr += 4;
		}
	}
}

TArray<Audio::FChannelPositionInfo>* FSoundFieldDecoder::GetDefaultChannelPositions(int32 InNumChannels)
{
	// TODO: Make default channel positions available through a static function. currently we have to retrieve the channel positions from the audio device.
	if (GEngine)
	{
		FAudioDevice* AudioDevice = GEngine->GetMainAudioDeviceRaw();
		Audio::FMixerDevice* MixerDevice = static_cast<Audio::FMixerDevice*>(AudioDevice);
		if (MixerDevice)
		{
			return MixerDevice->GetDefaultPositionMap(InNumChannels);
		}
	}

	return nullptr;
}

void FSoundFieldDecoder::FoaRotationInPlace(Audio::FAlignedFloatBuffer& InOutBuffer, const float XRotDegrees, const float YRotDegrees, const float ZRotDegrees)
{
	const int32 NumSamples = InOutBuffer.Num();
	const int32 NumFrames = NumSamples / 4; // FOA
	float* InOutBufferPtr = InOutBuffer.GetData();
	VectorRegister InputAmbiFrame = VectorRegister();

	FMatrix RotationMatrix;
	SphereHarmCalc.GenerateFirstOrderRotationMatrixGivenDegrees(-XRotDegrees + 180.0f, -YRotDegrees, ZRotDegrees, RotationMatrix);

	VectorRegister RotMatrixRow0 = VectorLoadAligned(RotationMatrix.M[0]);
	VectorRegister RotMatrixRow1 = VectorLoadAligned(RotationMatrix.M[1]);
	VectorRegister RotMatrixRow2 = VectorLoadAligned(RotationMatrix.M[2]);
	VectorRegister RotMatrixRow3 = VectorLoadAligned(RotationMatrix.M[3]);

	for (int32 i = 0; i < NumSamples; i += 4)
	{
		InputAmbiFrame = VectorLoadAligned(&InOutBufferPtr[i]);

		InOutBufferPtr[i + 0] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow0), 0);
		InOutBufferPtr[i + 1] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow1), 0);
		InOutBufferPtr[i + 2] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow2), 0);
		InOutBufferPtr[i + 3] = VectorGetComponent(VectorDot4(InputAmbiFrame, RotMatrixRow3), 0);
	}
}

// init static data
const VectorRegister FSoundFieldDecoder::Sqrt2Over2Vec = MakeVectorRegister(0.707f, 0.707f, 0.707f, 0.707f);
const VectorRegister FSoundFieldDecoder::ZeroVec = VectorZero();

FSphericalHarmonicCalculator FSoundFieldDecoder::SphereHarmCalc;

TArray<Audio::FChannelPositionInfo> FSoundFieldDecoder::VirtualSpeakerLocationsHorzOnly = {
	{EAudioMixerChannel::FrontLeft, 330, 0}
	,{EAudioMixerChannel::FrontRight, 30, 0}
	,{EAudioMixerChannel::FrontCenter, 0, 0}
	,{EAudioMixerChannel::LowFrequency, -1, 0}
	,{EAudioMixerChannel::BackLeft, 210, 0} 
	,{EAudioMixerChannel::BackRight, 150, 0}
	,{EAudioMixerChannel::SideLeft, 250, 0}
	,{EAudioMixerChannel::SideRight, 110, 0}
	,{EAudioMixerChannel::BackCenter, 180,0}
};

FSoundFieldEncoder::FSoundFieldEncoder()
{
}

void FSoundFieldEncoder::EncodeAudioDirectlyFromOutputPositions(const Audio::FAlignedFloatBuffer& InputData, const FSoundfieldSpeakerPositionalData& InputPositions, const FAmbisonicsSoundfieldSettings& Settings, FAmbisonicsSoundfieldBuffer& OutputData)
{
	constexpr const float DEG_2_RAD = PI / 180.0f;

	const int32 NumInputChannels = InputPositions.NumChannels;
	const int32 NumFrames = InputData.Num() / InputPositions.NumChannels;
	
	const int32 NumAmbiChannels = (Settings.Order + 1) * (Settings.Order + 1);

	check(NumInputChannels > 0 && NumAmbiChannels > 0);

	SpeakerGains.Reset();
	SpeakerGains.AddZeroed(NumInputChannels * NumAmbiChannels);

	OutputData.AudioBuffer.Reset();
	OutputData.AudioBuffer.AddZeroed(NumFrames * NumAmbiChannels);
	OutputData.NumChannels = NumAmbiChannels;
	OutputData.Rotation = FQuat::Identity;

	float* SpeakerGainsPtr = SpeakerGains.GetData();
	const float* InputBufferPtrBuffer = InputData.GetData();
	float* pAmbiFrame = OutputData.AudioBuffer.GetData();

	// gain normalization factor
	const float G = FMath::Sqrt(1.0f / static_cast<float>(NumAmbiChannels));


	// obtain listener orientation in spherical coordinates
	FVector2D ListenerRotationSphericalCoord = InputPositions.Rotation.Vector().UnitCartesianToSpherical();
	FSphericalHarmonicCalculator::AdjustUESphericalCoordinatesForAmbisonics(ListenerRotationSphericalCoord);

	check(InputPositions.ChannelPositions);

	const bool bIsMono = NumInputChannels == 1;

	// fill out the ambisonics speaker gain maps
	for (int InChan = 0; InChan < NumInputChannels; ++InChan)
	{
		const Audio::FChannelPositionInfo& CurrSpeakerPos = (*InputPositions.ChannelPositions)[InChan];

		// skip LFE and Center channel (leave gains at zero)
		// Mono audio channels are FrontCenter, so do _not_ skip if center channel and mono.
		const bool bSkipChannel = CurrSpeakerPos.Channel == EAudioMixerChannel::LowFrequency || (!bIsMono && (CurrSpeakerPos.Channel == EAudioMixerChannel::FrontCenter));
			
		if (bSkipChannel)
		{
			SpeakerGainsPtr += NumAmbiChannels;
			continue;
		}

		const float Theta = -(ListenerRotationSphericalCoord.Y + (float)CurrSpeakerPos.Azimuth * DEG_2_RAD);
		const float Phi = -(ListenerRotationSphericalCoord.X + (float)CurrSpeakerPos.Elevation * DEG_2_RAD);

		// SpeakerGainsPtr is assumed to be a pointer to a NumAmbiChannels-element array
		FSphericalHarmonicCalculator::ComputeSoundfieldChannelGains(Settings.Order, Theta, Phi, SpeakerGainsPtr);

		// apply gain normalization factor directly to speaker gains
		for (int i = 0; i < NumAmbiChannels; ++i)
		{
			SpeakerGainsPtr[i] *= G;
		}

		SpeakerGainsPtr += NumAmbiChannels;
	}

	// reset speaker gains pointer
	SpeakerGainsPtr = SpeakerGains.GetData();

	EncodeLoop(NumFrames, NumInputChannels, InputBufferPtrBuffer, NumAmbiChannels, SpeakerGainsPtr, pAmbiFrame);
}

void FSoundFieldEncoder::EncodeLoop(const int32 NumFrames, const int32 NumInputChannels, const float* RESTRICT InputAudioPtr, const int32 NumAmbiChannels, float* RESTRICT SpeakerGainsPtr, float* RESTRICT OutputAmbiBuffer)
{
	// Encoding to ambisonics is N^2.
	for (int32 FrameIndex = 0; FrameIndex < NumFrames; ++FrameIndex)
	{
		for (int32 AmbiChannelIndex = 0; AmbiChannelIndex < NumAmbiChannels; AmbiChannelIndex++)
		{
			const int32 OutAmbiSampleOffset = FrameIndex * NumAmbiChannels + AmbiChannelIndex;
			OutputAmbiBuffer[OutAmbiSampleOffset] = 0.0f;

			for (int32 InputChannelIndex = 0; InputChannelIndex < NumInputChannels; InputChannelIndex++)
			{
				float& CurrentSpeakerGain = SpeakerGainsPtr[InputChannelIndex * NumAmbiChannels + AmbiChannelIndex];
				OutputAmbiBuffer[OutAmbiSampleOffset] += InputAudioPtr[FrameIndex * NumInputChannels + InputChannelIndex] * CurrentSpeakerGain;
			}
		}
	}
}

void FAmbisonicsSoundfieldBuffer::Serialize(FArchive& Ar)
{
	Ar << AudioBuffer;
	Ar << NumChannels;
}

TUniquePtr<ISoundfieldAudioPacket> FAmbisonicsSoundfieldBuffer::Duplicate() const
{
	return TUniquePtr<ISoundfieldAudioPacket>(new FAmbisonicsSoundfieldBuffer(*this));
}

void FAmbisonicsSoundfieldBuffer::Reset()
{
	AudioBuffer.Reset();
	NumChannels = 0;
}

uint32 FAmbisonicsSoundfieldSettings::GetUniqueId() const
{
	// We can just use the Order directly.
	return Order;
}

TUniquePtr<ISoundfieldEncodingSettingsProxy> FAmbisonicsSoundfieldSettings::Duplicate() const
{
	FAmbisonicsSoundfieldSettings* Proxy = new FAmbisonicsSoundfieldSettings();
	Proxy->Order = Order;
	return TUniquePtr<ISoundfieldEncodingSettingsProxy>(Proxy);
}

FName GetUnrealAmbisonicsFormatName()
{
	static FName AmbisonicsSoundfieldFormatName = FName("Unreal Ambisonics");
	return AmbisonicsSoundfieldFormatName;
}

TUniquePtr<ISoundfieldDecoderStream> CreateDefaultSourceAmbisonicsDecoder(Audio::FMixerDevice* InDevice)
{
	check(InDevice);

	ISoundfieldFactory* UnrealAmbisonicsFactory = ISoundfieldFactory::Get(GetUnrealAmbisonicsFormatName());
	if (UnrealAmbisonicsFactory)
	{
		FAudioPluginInitializationParams InitParams;
		InitParams.AudioDevicePtr = InDevice;
		InitParams.BufferLength = InDevice->GetNumOutputFrames();
		InitParams.NumOutputChannels = InDevice->GetNumDeviceChannels();
		InitParams.NumSources = InDevice->GetMaxSources();
		InitParams.SampleRate = InDevice->GetSampleRate();
		return UnrealAmbisonicsFactory->CreateDecoderStream(InitParams, GetAmbisonicsSourceDefaultSettings());
	}
	else
	{
		return nullptr;
	}
}

SOUNDFIELDRENDERING_API ISoundfieldEncodingSettingsProxy& GetAmbisonicsSourceDefaultSettings()
{
	static FAmbisonicsSoundfieldSettings DefaultSettings;
	DefaultSettings.Order = 1;

	return DefaultSettings;
}
