// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ISoundfieldFormat.h"
#include "DSP/SphericalHarmonicCalculator.h"
#include "DSP/BufferVectorOperations.h"

SOUNDFIELDRENDERING_API FName GetUnrealAmbisonicsFormatName();
SOUNDFIELDRENDERING_API TUniquePtr<ISoundfieldDecoderStream> CreateDefaultSourceAmbisonicsDecoder(Audio::FMixerDevice* InDevice);
SOUNDFIELDRENDERING_API ISoundfieldEncodingSettingsProxy& GetAmbisonicsSourceDefaultSettings();


class FAmbisonicsSoundfieldBuffer : public ISoundfieldAudioPacket
{
public:
	// Interleaved audio buffer for all vector parts of the ambisonics stream.
	Audio::FAlignedFloatBuffer AudioBuffer;

	// number of channels in the Ambisonics stream.
	// Currently we don't explicitly support mixed order ambisonics,
	// so this will always be equal to (m + 1)^2, where m is the order  of ambisonics this was encoded with.
	int32 NumChannels;

	// This is the rotation of the ambisonics source.
	FQuat Rotation;
	FQuat PreviousRotation;

	FAmbisonicsSoundfieldBuffer()
		: NumChannels(0)
		, Rotation(FQuat::Identity)
		, PreviousRotation(FQuat::Identity)
	{}

	virtual ~FAmbisonicsSoundfieldBuffer() = default;

	SOUNDFIELDRENDERING_API virtual void Serialize(FArchive& Ar) override;
	SOUNDFIELDRENDERING_API virtual TUniquePtr<ISoundfieldAudioPacket> Duplicate() const override;
	SOUNDFIELDRENDERING_API virtual void Reset() override;

};

class FAmbisonicsSoundfieldSettings : public ISoundfieldEncodingSettingsProxy
{
public:
	int32 Order;

	SOUNDFIELDRENDERING_API virtual uint32 GetUniqueId() const override;
	SOUNDFIELDRENDERING_API virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> Duplicate() const override;

};

class FSoundFieldDecoder
{
public:
	SOUNDFIELDRENDERING_API FSoundFieldDecoder(); // initializes VirtualSpeakerWorldLockedSpeakerGains

	SOUNDFIELDRENDERING_API void DecodeAudioDirectlyToDeviceOutputPositions(const FAmbisonicsSoundfieldBuffer& InputData, const FSoundfieldSpeakerPositionalData& OutputPositions, Audio::FAlignedFloatBuffer& OutputData);
	SOUNDFIELDRENDERING_API void DecodeAudioToSevenOneAndDownmixToDevice(const FAmbisonicsSoundfieldBuffer& InputData, const FSoundfieldSpeakerPositionalData& OutputPositions, Audio::FAlignedFloatBuffer& OutputData);
	static SOUNDFIELDRENDERING_API void RotateFirstOrderAmbisonicsBed(const FAmbisonicsSoundfieldBuffer& InputData, FAmbisonicsSoundfieldBuffer& OutputData, const FQuat& DestinationRotation, const FQuat& PreviousRotation);

	static SOUNDFIELDRENDERING_API void FoaRotationInPlace(Audio::FAlignedFloatBuffer& InOutFrames, const float XRotDegrees, const float YRotDegrees, const float ZRotDegrees);

private:
	// Special cased, vectorized versions of first order operations.
	static SOUNDFIELDRENDERING_API void FirstOrderDecodeLoop(const int32 NumFrames, const int32 NumOutputChannels, const float* pAmbiFrame, const int32 NumAmbiChannels, float* SpeakerGainsPtr, float* OutputBufferPtrBuffer);

	// Special cased vectorized versions of 3rd and fifth order operations.
	static SOUNDFIELDRENDERING_API void OddOrderDecodeLoop(const int32 NumFrames, const int32 NumOutputChannels, const float* pAmbiFrame, const int32 NumAmbiChannels, float* SpeakerGainsPtr, float* OutputBufferPtrBuffer);

	// Partially vectorized decode operations for 2nd and 4th order operations.
	static SOUNDFIELDRENDERING_API void EvenOrderDecodeLoop(const int32 NumFrames, const int32 NumOutputChannels, const float* pAmbiFrame, const int32 NumAmbiChannels, float* SpeakerGainsPtr, float* OutputBufferPtrBuffer);

	// Helper function to get the default channel positions for a channel array.
	UE_DEPRECATED(5.8, "Channel position info can be retrieved from the FMixerDevice")
	static SOUNDFIELDRENDERING_API TArray<Audio::FChannelPositionInfo>* GetDefaultChannelPositions(int32 InNumChannels);

	Audio::FAlignedFloatBuffer VirtualSpeakerScratchBuffers;
	Audio::FAlignedFloatBuffer FoaVirtualSpeakerWordLockedGains;
	Audio::FAlignedFloatBuffer TargetSpeakerGains;
	Audio::FAlignedFloatBuffer CurrentSpeakerGains;
	Audio::FAlignedFloatBuffer MixdownGainsMap;
	Audio::FBufferLinearEase SpeakerGainLerper;
	FVector2D LastListenerRotationSphericalCoord{ -999.f, -999.f }; // force interpolation on first callback

	static SOUNDFIELDRENDERING_API FSphericalHarmonicCalculator SphereHarmCalc;
	static SOUNDFIELDRENDERING_API TArray<Audio::FChannelPositionInfo> VirtualSpeakerLocationsHorzOnly;
	static SOUNDFIELDRENDERING_API const VectorRegister Sqrt2Over2Vec;
	static SOUNDFIELDRENDERING_API const VectorRegister ZeroVec;

	
	friend class FSoundFieldEncoder;
};

class FSoundFieldEncoder
{
public:
	SOUNDFIELDRENDERING_API FSoundFieldEncoder();
	SOUNDFIELDRENDERING_API void EncodeAudioDirectlyFromOutputPositions(const Audio::FAlignedFloatBuffer& InputData, const FSoundfieldSpeakerPositionalData& InputPositions, const FAmbisonicsSoundfieldSettings& Settings, FAmbisonicsSoundfieldBuffer& OutputData);

private:
	static void EncodeLoop(const int32 NumFrames, const int32 NumInputChannels, const float* RESTRICT InputAudioPtr, const int32 NumAmbiChannels, float* RESTRICT SpeakerGainsPtr, float* RESTRICT OutputAmbiBuffer);

	Audio::FAlignedFloatBuffer VirtualSpeakerScratchBuffers;
	Audio::FAlignedFloatBuffer SpeakerGains;
	Audio::FAlignedFloatBuffer MixdownGainsMap;
};
