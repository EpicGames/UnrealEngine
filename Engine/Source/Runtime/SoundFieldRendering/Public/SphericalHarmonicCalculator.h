// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

// This file has moved.
UE_DEPRECATED_HEADER(5.7, "This file has moved to the SignalProcessingModule")
#include "DSP/SphericalHarmonicCalculator.h"
