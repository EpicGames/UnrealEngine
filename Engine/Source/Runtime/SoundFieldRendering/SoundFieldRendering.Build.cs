// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class SoundFieldRendering : ModuleRules
	{
		public SoundFieldRendering(ReadOnlyTargetRules Target) : base(Target)
		{
            PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Engine",
					"SignalProcessing",
					"AudioMixer",
					"AudioExtensions"
				}
			);

			// This is used to get FSoundQualityInfo struct for IAudioEncoder.
            PrivateIncludePathModuleNames.Add("TargetPlatform");

			// This is used to reference the EAudioMixerChannel enumeration.
            PrivateIncludePathModuleNames.Add("AudioMixerCore");
        }
	}
}
